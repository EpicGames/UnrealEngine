{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Analytics Blueprint Library",
	"Description" : "Blueprint Library for using an analytic event provider",
	"Category" : "Analytics",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "https://docs.unrealengine.com/latest/INT/Gameplay/Analytics/Blueprints/index.html",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "AnalyticsBlueprintLibrary",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault"
		}
	]
}