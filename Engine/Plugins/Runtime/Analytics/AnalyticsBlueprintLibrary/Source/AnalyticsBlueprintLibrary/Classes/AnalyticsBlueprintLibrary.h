// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "AnalyticsBlueprintLibrary.generated.h"

class Error;

/** Blueprint accessible version of the analytics event struct */
USTRUCT(BlueprintType)
struct FAnalyticsEventAttr
{
	GENERATED_USTRUCT_BODY();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Analytics")
	FString Name;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Analytics")
	FString Value;
};

UCLASS(meta=(ScriptName="AnalyticsLibrary"))
class UAnalyticsBlueprintLibrary :
	public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/** Starts an analytics session without any custom attributes specified */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static bool StartSession();

	/** Starts an analytics session with custom attributes specified */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static bool StartSessionWithAttributes(const TArray<FAnalyticsEventAttr>& Attributes);

	/** Ends an analytics session */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void EndSession();

	/** Requests that any cached events be sent immediately */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void FlushEvents();

	/** Records an event has happened by name without any attributes (an event counter) */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void RecordEvent(const FString& EventName);

	/** Records an event has happened by name with a single attribute */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void RecordEventWithAttribute(const FString& EventName, const FString& AttributeName, const FString& AttributeValue);

	/** Records an event has happened by name with a single attribute */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void RecordEventWithAttributes(const FString& EventName, const TArray<FAnalyticsEventAttr>& Attributes);

	/** Records an in-game item was purchased using the specified in-game currency */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void RecordItemPurchase(const FString& ItemId, const FString& Currency, int32 PerItemCost, int32 ItemQuantity);

	/** Records an in-game item was purchased */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void RecordSimpleItemPurchase(const FString& ItemId, int32 ItemQuantity);

	/** Records an in-game item was purchased with attributes */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void RecordSimpleItemPurchaseWithAttributes(const FString& ItemId, int32 ItemQuantity, const TArray<FAnalyticsEventAttr>& Attributes);

	/** Records an in-game currency was purchased using real-world money */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void RecordSimpleCurrencyPurchase(const FString& GameCurrencyType, int32 GameCurrencyAmount);

	/** Records an in-game currency was purchased using real-world money */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void RecordSimpleCurrencyPurchaseWithAttributes(const FString& GameCurrencyType, int32 GameCurrencyAmount, const TArray<FAnalyticsEventAttr>& Attributes);

	/** Records an in-game currency was purchased using real-world money */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void RecordCurrencyPurchase(const FString& GameCurrencyType, int32 GameCurrencyAmount, const FString& RealCurrencyType, float RealMoneyCost, const FString& PaymentProvider);

	/** Records an in-game currency was granted by the game with no real-world money being involved */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void RecordCurrencyGiven(const FString& GameCurrencyType, int32 GameCurrencyAmount);

	/** Records an in-game currency was granted by the game with no real-world money being involved */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void RecordCurrencyGivenWithAttributes(const FString& GameCurrencyType, int32 GameCurrencyAmount, const TArray<FAnalyticsEventAttr>& Attributes);

	/** Builds a struct from the attribute name and value */
	UFUNCTION(BlueprintPure, Category="Analytics")
	static FAnalyticsEventAttr MakeEventAttribute(const FString& AttributeName, const FString& AttributeValue);

	/** Gets the current session id from the analytics provider */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static FString GetSessionId();

	/** Sets the session id (if supported) on the analytics provider */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void SetSessionId(const FString& SessionId);

	/** Gets the current user id from the analytics provider */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static FString GetUserId();

	/** Sets the user id (if supported) on the analytics provider */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void SetUserId(const FString& UserId);

	/** Sets the user's age (if supported) on the analytics provider */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void SetAge(int32 Age);

	/** Sets the user's location (if supported) on the analytics provider */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void SetLocation(const FString& Location);

	/** Sets the user's gender (if supported) on the analytics provider */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void SetGender(const FString& Gender);

	/** Sets the game's build info (if supported) on the analytics provider */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void SetBuildInfo(const FString& BuildInfo);

	/** Records an error event has happened with attributes */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void RecordErrorWithAttributes(const FString& Error, const TArray<FAnalyticsEventAttr>& Attributes);

	/** Records an error event has happened */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void RecordError(const FString& Error);

	/** Records a user progress event has happened with a full list of progress hierarchy names and with attributes */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void RecordProgressWithFullHierarchyAndAttributes(const FString& ProgressType, const TArray<FString>& ProgressNames, const TArray<FAnalyticsEventAttr>& Attributes);

	/** Records a user progress event has happened with attributes */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void RecordProgressWithAttributes(const FString& ProgressType, const FString& ProgressName, const TArray<FAnalyticsEventAttr>& Attributes);

	/** Records a user progress event has happened */
	UFUNCTION(BlueprintCallable, Category="Analytics")
	static void RecordProgress(const FString& ProgressType, const FString& ProgressName);
};
