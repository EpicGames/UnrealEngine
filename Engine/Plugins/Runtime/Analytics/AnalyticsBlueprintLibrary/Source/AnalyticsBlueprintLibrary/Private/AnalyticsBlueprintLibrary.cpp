// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyticsBlueprintLibrary.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "Analytics.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnalyticsBlueprintLibrary)

DEFINE_LOG_CATEGORY_STATIC(LogAnalyticsBPLib, Display, All);

/**
 * Converts the UObject accessible array into the native only analytics array type
 */
static inline TArray<FAnalyticsEventAttribute> ConvertAttrs(const TArray<FAnalyticsEventAttr>& Attributes)
{
	TArray<FAnalyticsEventAttribute> Converted;
	Converted.Reserve(Attributes.Num());
	for (const FAnalyticsEventAttr& Attr : Attributes)
	{
		Converted.Emplace(Attr.Name, Attr.Value);
	}
	return Converted;
}


UAnalyticsBlueprintLibrary::UAnalyticsBlueprintLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UAnalyticsBlueprintLibrary::StartSession()
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		return Provider->StartSession();
	}
	else
	{
		UE_LOGF(LogAnalyticsBPLib, Warning, "StartSession: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
	}
	return false;
}

bool UAnalyticsBlueprintLibrary::StartSessionWithAttributes(const TArray<FAnalyticsEventAttr>& Attributes)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		return Provider->StartSession(ConvertAttrs(Attributes));
	}
	else
	{
		UE_LOGF(LogAnalyticsBPLib, Warning, "StartSessionWithAttributes: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
	}
	return false;
}

void UAnalyticsBlueprintLibrary::EndSession()
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->EndSession();
	}
	else
	{
		UE_LOGF(LogAnalyticsBPLib, Warning, "EndSession: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
	}
}

void UAnalyticsBlueprintLibrary::FlushEvents()
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->FlushEvents();
	}
	else
	{
		UE_LOGF(LogAnalyticsBPLib, Warning, "FlushEvents: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
	}
}

void UAnalyticsBlueprintLibrary::RecordEvent(const FString& EventName)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->RecordEvent(EventName);
	}
	else
	{
		static bool bHasLogged = false;
		if (!bHasLogged)
		{
			UE_LOGF(LogAnalyticsBPLib, Warning, "RecordEvent: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
		}
	}
}

void UAnalyticsBlueprintLibrary::RecordEventWithAttribute(const FString& EventName, const FString& AttributeName, const FString& AttributeValue)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		FAnalyticsEventAttribute Attribute(AttributeName, AttributeValue);
		Provider->RecordEvent(EventName, Attribute);
	}
	else
	{
		static bool bHasLogged = false;
		if (!bHasLogged)
		{
			UE_LOGF(LogAnalyticsBPLib, Warning, "RecordEventWithAttribute: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
		}
	}
}

void UAnalyticsBlueprintLibrary::RecordEventWithAttributes(const FString& EventName, const TArray<FAnalyticsEventAttr>& Attributes)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->RecordEvent(EventName, ConvertAttrs(Attributes));
	}
	else
	{
		static bool bHasLogged = false;
		if (!bHasLogged)
		{
			UE_LOGF(LogAnalyticsBPLib, Warning, "RecordEventWithAttributes: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
		}
	}
}

void UAnalyticsBlueprintLibrary::RecordItemPurchase(const FString& ItemId, const FString& Currency, int32 PerItemCost, int32 ItemQuantity)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->RecordItemPurchase(ItemId, Currency, PerItemCost, ItemQuantity);
	}
	else
	{
		static bool bHasLogged = false;
		if (!bHasLogged)
		{
			UE_LOGF(LogAnalyticsBPLib, Warning, "RecordItemPurchase: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
		}
	}
}

void UAnalyticsBlueprintLibrary::RecordSimpleItemPurchase(const FString& ItemId, int32 ItemQuantity)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->RecordItemPurchase(ItemId, ItemQuantity);
	}
	else
	{
		static bool bHasLogged = false;
		if (!bHasLogged)
		{
			UE_LOGF(LogAnalyticsBPLib, Warning, "RecordSimpleItemPurchase: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
		}
	}
}

void UAnalyticsBlueprintLibrary::RecordSimpleItemPurchaseWithAttributes(const FString& ItemId, int32 ItemQuantity, const TArray<FAnalyticsEventAttr>& Attributes)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->RecordItemPurchase(ItemId, ItemQuantity, ConvertAttrs(Attributes));
	}
	else
	{
		static bool bHasLogged = false;
		if (!bHasLogged)
		{
			UE_LOGF(LogAnalyticsBPLib, Warning, "RecordSimpleItemPurchaseWithAttributes: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
		}
	}
}

void UAnalyticsBlueprintLibrary::RecordSimpleCurrencyPurchase(const FString& GameCurrencyType, int32 GameCurrencyAmount)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->RecordCurrencyPurchase(GameCurrencyType, GameCurrencyAmount);
	}
	else
	{
		UE_LOGF(LogAnalyticsBPLib, Warning, "RecordSimpleCurrencyPurchase: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
	}
}

void UAnalyticsBlueprintLibrary::RecordSimpleCurrencyPurchaseWithAttributes(const FString& GameCurrencyType, int32 GameCurrencyAmount, const TArray<FAnalyticsEventAttr>& Attributes)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->RecordCurrencyPurchase(GameCurrencyType, GameCurrencyAmount, ConvertAttrs(Attributes));
	}
	else
	{
		UE_LOGF(LogAnalyticsBPLib, Warning, "RecordSimpleCurrencyPurchaseWithAttributes: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
	}
}

void UAnalyticsBlueprintLibrary::RecordCurrencyPurchase(const FString& GameCurrencyType, int32 GameCurrencyAmount, const FString& RealCurrencyType, float RealMoneyCost, const FString& PaymentProvider)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->RecordCurrencyPurchase(GameCurrencyType, GameCurrencyAmount, RealCurrencyType, RealMoneyCost, PaymentProvider);
	}
	else
	{
		UE_LOGF(LogAnalyticsBPLib, Warning, "RecordCurrencyPurchase: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
	}
}

void UAnalyticsBlueprintLibrary::RecordCurrencyGiven(const FString& GameCurrencyType, int32 GameCurrencyAmount)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->RecordCurrencyGiven(GameCurrencyType, GameCurrencyAmount);
	}
	else
	{
		UE_LOGF(LogAnalyticsBPLib, Warning, "RecordCurrencyGiven: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
	}
}

void UAnalyticsBlueprintLibrary::RecordCurrencyGivenWithAttributes(const FString& GameCurrencyType, int32 GameCurrencyAmount, const TArray<FAnalyticsEventAttr>& Attributes)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->RecordCurrencyGiven(GameCurrencyType, GameCurrencyAmount, ConvertAttrs(Attributes));
	}
	else
	{
		UE_LOGF(LogAnalyticsBPLib, Warning, "RecordCurrencyGivenWithAttributes: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
	}
}

FString UAnalyticsBlueprintLibrary::GetSessionId()
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		return Provider->GetSessionID();
	}
	return FString();
}

void UAnalyticsBlueprintLibrary::SetSessionId(const FString& SessionId)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->SetSessionID(SessionId);
	}
	else
	{
		UE_LOGF(LogAnalyticsBPLib, Warning, "SetSessionId: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
	}
}

FString UAnalyticsBlueprintLibrary::GetUserId()
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		return Provider->GetUserID();
	}
	return FString();
}

void UAnalyticsBlueprintLibrary::SetUserId(const FString& UserId)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->SetUserID(UserId);
	}
	else
	{
		UE_LOGF(LogAnalyticsBPLib, Warning, "SetUserId: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
	}
}

FAnalyticsEventAttr UAnalyticsBlueprintLibrary::MakeEventAttribute(const FString& AttributeName, const FString& AttributeValue)
{
	FAnalyticsEventAttr EventAttr;
	EventAttr.Name = AttributeName;
	EventAttr.Value = AttributeValue;
	return EventAttr;
}

void UAnalyticsBlueprintLibrary::SetAge(int32 Age)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->SetAge(Age);
	}
	else
	{
		UE_LOGF(LogAnalyticsBPLib, Warning, "SetAge: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
	}
}

void UAnalyticsBlueprintLibrary::SetLocation(const FString& Location)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->SetLocation(Location);
	}
	else
	{
		UE_LOGF(LogAnalyticsBPLib, Warning, "SetLocation: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
	}
}

void UAnalyticsBlueprintLibrary::SetGender(const FString& Gender)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->SetGender(Gender);
	}
	else
	{
		UE_LOGF(LogAnalyticsBPLib, Warning, "SetGender: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
	}
}

void UAnalyticsBlueprintLibrary::SetBuildInfo(const FString& BuildInfo)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->SetBuildInfo(BuildInfo);
	}
	else
	{
		UE_LOGF(LogAnalyticsBPLib, Warning, "SetBuildInfo: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
	}
}

void UAnalyticsBlueprintLibrary::RecordErrorWithAttributes(const FString& Error, const TArray<FAnalyticsEventAttr>& Attributes)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->RecordError(Error, ConvertAttrs(Attributes));
	}
	else
	{
		UE_LOGF(LogAnalyticsBPLib, Warning, "RecordErrorWithAttributes: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
	}
}

void UAnalyticsBlueprintLibrary::RecordError(const FString& Error)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->RecordError(Error);
	}
	else
	{
		UE_LOGF(LogAnalyticsBPLib, Warning, "RecordError: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
	}
}

void UAnalyticsBlueprintLibrary::RecordProgressWithFullHierarchyAndAttributes(const FString& ProgressType, const TArray<FString>& ProgressNames, const TArray<FAnalyticsEventAttr>& Attributes)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->RecordProgress(ProgressType, ProgressNames, ConvertAttrs(Attributes));
	}
	else
	{
		UE_LOGF(LogAnalyticsBPLib, Warning, "RecordProgressWithFullHierarchyAndAttributes: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
	}
}

void UAnalyticsBlueprintLibrary::RecordProgressWithAttributes(const FString& ProgressType, const FString& ProgressName, const TArray<FAnalyticsEventAttr>& Attributes)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->RecordProgress(ProgressType, ProgressName, ConvertAttrs(Attributes));
	}
	else
	{
		UE_LOGF(LogAnalyticsBPLib, Warning, "RecordProgressWithAttributes: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
	}
}

void UAnalyticsBlueprintLibrary::RecordProgress(const FString& ProgressType, const FString& ProgressName)
{
	TSharedPtr<IAnalyticsProvider> Provider = FAnalytics::Get().GetDefaultConfiguredProvider();
	if (Provider.IsValid())
	{
		Provider->RecordProgress(ProgressType, ProgressName);
	}
	else
	{
		UE_LOGF(LogAnalyticsBPLib, Warning, "RecordProgress: Failed to get the default analytics provider. Double check your [Analytics] configuration in your INI");
	}
}

