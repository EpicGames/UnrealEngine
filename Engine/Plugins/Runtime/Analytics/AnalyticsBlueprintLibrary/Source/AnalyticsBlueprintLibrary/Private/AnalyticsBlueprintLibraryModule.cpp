// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyticsBlueprintLibraryModule.h"

IMPLEMENT_MODULE( FAnalyticsBlueprintLibraryModule, AnalyticsBlueprintLibrary )

void FAnalyticsBlueprintLibraryModule::StartupModule()
{
}

void FAnalyticsBlueprintLibraryModule::ShutdownModule()
{
}
